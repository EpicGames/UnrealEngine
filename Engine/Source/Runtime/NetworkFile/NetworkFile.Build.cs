// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NetworkFile : ModuleRules
{
	public NetworkFile(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.Add("DerivedDataCache");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Sockets",
				"CookOnTheFly"
			});

		PublicDefinitions.Add("ENABLE_HTTP_FOR_NETWORK_FILE=0");

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
