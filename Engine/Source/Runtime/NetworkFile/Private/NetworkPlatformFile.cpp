// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkPlatformFile.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Misc/ScopedEvent.h"
#include "HAL/ThreadSafeCounter.h"
#include "Misc/ScopeLock.h"
#include "Stats/StatsMisc.h"
#include "Stats/Stats.h"
#include "Async/AsyncWork.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/LocalTimestampDirectoryVisitor.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "DerivedDataCacheInterface.h"
#include "Misc/PackageName.h"
#include "Misc/PathViews.h"
#include "SocketSubsystem.h"

#include "HAL/IPlatformFileModule.h"
#include "Templates/UniquePtr.h"

#include "UObject/Object.h"
#include "CookOnTheFly.h"
#include "CookOnTheFlyMessages.h"

DEFINE_LOG_CATEGORY(LogNetworkPlatformFile);

FString FNetworkPlatformFile::MP4Extension = TEXT(".mp4");
FString FNetworkPlatformFile::BulkFileExtension = TEXT(".ubulk");
FString FNetworkPlatformFile::ExpFileExtension = TEXT(".uexp");
FString FNetworkPlatformFile::FontFileExtension = TEXT(".ufont");

// These are marked unsafe because they do not work with Programs. However, COTF is unlikely to be used with Programs
// These are also temporary until some issues can be debugged
static FString UnsafeEnginePlatformExtensionDir()
{
	return FPaths::EnginePlatformExtensionDir(TEXT("")).TrimChar('/');
}

static FString UnsafeProjectPlatformExtensionDir()
{
	return FPaths::ProjectPlatformExtensionDir(TEXT("")).TrimChar('/');
}

FNetworkPlatformFile::FNetworkPlatformFile()
	: InnerPlatformFile(NULL)
	, bIsUsable(false)
	, FileServerPort(0)
	, ConnectionFlags(EConnectionFlags::None)
	, HeartbeatFrequency(5.0f)
	, FinishedAsyncNetworkReadUnsolicitedFiles(NULL)
	, FinishedAsyncWriteUnsolicitedFiles(NULL)
{
	TotalWriteTime = 0.0; // total non async time spent writing to disk
	TotalNetworkSyncTime = 0.0; // total non async time spent syncing to network
	TotalTimeSpentInUnsolicitedPackages = 0.0; // total time async processing unsolicited packages
	TotalWaitForAsyncUnsolicitedPackages = 0.0; // total time spent waiting for unsolicited packages
	TotalFilesSynced = 0; // total number files synced from network
	TotalFilesFoundLocally = 0;
	TotalUnsolicitedPackages = 0; // total number unsolicited files synced  
	UnsolicitedPackagesHits = 0; // total number of hits from waiting on unsolicited packages
	UnsolicitedPackageWaits = 0; // total number of waits on unsolicited packages
}

bool FNetworkPlatformFile::ShouldBeUsed(IPlatformFile* Inner, const TCHAR* CmdLine) const
{
	FString HostIp;
	if (!FParse::Value(CmdLine, TEXT("-FileHostIP="), HostIp))
	{
		return false;
	}
	UE::Cook::ICookOnTheFlyModule& CookOnTheFlyModule = FModuleManager::LoadModuleChecked<UE::Cook::ICookOnTheFlyModule>(TEXT("CookOnTheFly"));
	TSharedPtr<UE::Cook::ICookOnTheFlyServerConnection> DefaultConnection = CookOnTheFlyModule.GetDefaultServerConnection();
	return !DefaultConnection.IsValid() || DefaultConnection->GetZenProjectName().IsEmpty();
}

bool FNetworkPlatformFile::Initialize(IPlatformFile* Inner, const TCHAR* CmdLine)
{
	UE::Cook::ICookOnTheFlyModule& CookOnTheFlyModule = FModuleManager::LoadModuleChecked<UE::Cook::ICookOnTheFlyModule>(TEXT("CookOnTheFly"));
	Connection = CookOnTheFlyModule.GetDefaultServerConnection();
	if (!Connection.IsValid())
	{
		UE_LOGF(LogNetworkPlatformFile, Warning, "No COTF server connection.");
		return false;
	}
	if (!InitializeInternal(Inner, *Connection->GetHost()))
	{
		UE_LOGF(LogNetworkPlatformFile, Warning, "Failed to initialize %ls.", *Connection->GetHost());
		return false;
	}
	Connection->OnMessage().AddRaw(this, &FNetworkPlatformFile::OnCookOnTheFlyMessage);
	return true;
}

bool FNetworkPlatformFile::InitializeInternal(IPlatformFile* Inner, const TCHAR* HostIP)
{
	// This platform file requires an inner.
	check(Inner != NULL);
	InnerPlatformFile = Inner;
	if (HostIP == NULL)
	{
		UE_LOGF(LogNetworkPlatformFile, Error, "No Host IP specified in the commandline.");
		bIsUsable = false;
		return false;
	}

	// Save and Intermediate directories are always local
	LocalDirectories.Add(FPaths::EngineDir() / TEXT("Binaries"));
	LocalDirectories.Add(FPaths::EngineIntermediateDir());
	LocalDirectories.Add(FPaths::ProjectDir() / TEXT("Binaries"));
	LocalDirectories.Add(FPaths::ProjectIntermediateDir());
	LocalDirectories.Add(FPaths::ProjectSavedDir() / TEXT("Backup"));
	LocalDirectories.Add(FPaths::ProjectSavedDir() / TEXT("Config"));
	LocalDirectories.Add(FPaths::ProjectSavedDir() / TEXT("Logs"));
	LocalDirectories.Add(FPaths::ProjectSavedDir() / TEXT("Sandboxes"));

	if (InnerPlatformFile->GetLowerLevel())
	{
		InnerPlatformFile->GetLowerLevel()->AddLocalDirectories(LocalDirectories);
	}
	else
	{
		InnerPlatformFile->AddLocalDirectories(LocalDirectories);
	}
	

	FNetworkFileArchive Payload(NFS_Messages::Heartbeat); 
	FArrayReader Out;
	if (!SendPayloadAndReceiveResponse(Payload,Out))
		bIsUsable = true; 


	// lets see we can test whether the server is up. 
	if (Out.Num())
	{
		FCommandLine::AddToSubprocessCommandLine( *FString::Printf( TEXT("-FileHostIP=%s"), HostIP ), ECommandLineArgumentFlags::AllContexts );
		bIsUsable = true; 
	}
	return bIsUsable;
}

bool FNetworkPlatformFile::SendPayloadAndReceiveResponse(TArray<uint8>& In, TArray<uint8>& Out)
{
	using namespace UE::Cook;
	using namespace UE::ZenCookOnTheFly::Messaging;

	{
		FScopeLock ScopeLock(&SynchronizationObject);
		if ( FinishedAsyncNetworkReadUnsolicitedFiles )
		{
			delete FinishedAsyncNetworkReadUnsolicitedFiles;
			FinishedAsyncNetworkReadUnsolicitedFiles = NULL;
		}
	}

	FCookOnTheFlyRequest Request(ECookOnTheFlyMessage::NetworkPlatformFile);
	TUniquePtr<FArchive> RequestPayload = Request.WriteBody();
	RequestPayload->Serialize(In.GetData(), In.Num());
	FCookOnTheFlyResponse Response = Connection->SendRequest(Request).Get();

	if (!Response.IsOk())
	{
		UE_LOGF(LogCookOnTheFly, Warning, "Failed to send 'NetworkPlatformFile' request");
		return false;
	}

	TUniquePtr<FArchive> ResponsePayload = Response.ReadBody();
	
	if (!IntFitsIn<int32, int64>(ResponsePayload->TotalSize()))
	{
		UE_LOGF(LogCookOnTheFly, Warning, "Failed to parse 'CookOnTheFlyResponse' because the payload was too large");
		return false;
	}
	
	Out.SetNum(static_cast<int32>(ResponsePayload->TotalSize()));
	ResponsePayload->Serialize(Out.GetData(), Out.Num());
	return true;
}

void FNetworkPlatformFile::OnCookOnTheFlyMessage(const UE::Cook::FCookOnTheFlyMessage& Message)
{
	using namespace UE::Cook;
	using namespace UE::ZenCookOnTheFly::Messaging;

	if (Message.GetHeader().MessageType == ECookOnTheFlyMessage::NetworkPlatformFile)
	{
		TUniquePtr<FArchive> PayloadReader = Message.ReadBody();
		TArray<uint8> Payload;

		if (!IntFitsIn<int32, int64>(PayloadReader->TotalSize()))
		{
			UE_LOGF(LogCookOnTheFly, Warning, "Failed to parse 'CookOnTheFlyMessage' because the payload was too large");
			return;
		}

		Payload.SetNum(static_cast<int32>(PayloadReader->TotalSize()));
		PayloadReader->Serialize(Payload.GetData(), Payload.Num());
		PendingPayloads.Enqueue(MoveTemp(Payload));
		NewPayloadEvent->Trigger();
	}
}

bool FNetworkPlatformFile::ReceiveResponse(TArray<uint8> &Out )
{
	for (;;)
	{
		NewPayloadEvent->Wait();
		TOptional<TArray<uint8>> Payload = PendingPayloads.Dequeue();
		if (Payload.IsSet())
		{
			Out = MoveTemp(Payload.GetValue());
			break;
		}
	}
	return true;
}


void FNetworkPlatformFile::InitializeAfterSetActive()
{
	double NetworkFileStartupTime = 0.0;
	{
		SCOPE_SECONDS_COUNTER(NetworkFileStartupTime);

		// send the filenames and timestamps to the server
		FNetworkFileArchive Payload(NFS_Messages::GetFileList);
		FillGetFileList(Payload);

		// send the directories over, and wait for a response
		FArrayReader Response;
		if (!SendPayloadAndReceiveResponse(Payload, Response))
		{
			Connection.Reset();
			return; 
		}
		else
		{
			// receive the cooked version information
			FPackageFileVersion ServerPackageVersion;
			int32 ServerPackageLicenseeVersion = 0;
			ProcessServerInitialResponse(Response, ServerPackageVersion, ServerPackageLicenseeVersion);
			ProcessServerCachedFilesResponse(Response, ServerPackageVersion, ServerPackageLicenseeVersion);

			// make sure we can sync a file
			FString TestSyncFile = FPaths::Combine(*(FPaths::EngineDir()), TEXT("Config/BaseEngine.ini"));

			InnerPlatformFile->SetReadOnly(*TestSyncFile, false);
			InnerPlatformFile->DeleteFile(*TestSyncFile);
			if (InnerPlatformFile->FileExists(*TestSyncFile))
			{
				UE_LOGF(LogNetworkPlatformFile, Error, "Could not delete file sync test file %ls.", *TestSyncFile);
			}

			EnsureFileIsLocal(TestSyncFile);

			if (!InnerPlatformFile->FileExists(*TestSyncFile) || InnerPlatformFile->FileSize(*TestSyncFile) < 1)
			{
				UE_LOGF(LogNetworkPlatformFile, Fatal, "Could not sync test file %ls.", *TestSyncFile);
			}
		}
	}

	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Network file startup time: %5.3f seconds\n"), NetworkFileStartupTime);

}

void FNetworkPlatformFile::ProcessServerCachedFilesResponse(FArrayReader& Response, const FPackageFileVersion& ServerPackageVersion, const int32 ServerPackageLicenseeVersion)
{
	/* The server root content directories */
	TArray<FString> ServerRootContentDirectories;
	Response << ServerRootContentDirectories;

	// receive a list of the cache files and their timestamps
	TMap<FString, FDateTime> ServerCachedFiles;
	{
		TMap<FString, FDateTime> ServerCachedFileResponse;
		Response << ServerCachedFileResponse;

		ServerCachedFiles.Reserve(ServerCachedFileResponse.Num());
		for (const auto& ServerFileTimePair : ServerCachedFileResponse)
		{
			FString Filename = ServerFileTimePair.Key;
			ConvertServerFilenameToClientFilename(Filename);
			Filename.ToLowerInline();
			ServerCachedFiles.Emplace(MoveTemp(Filename), ServerFileTimePair.Value);
		}
	}

	UE_LOGF(LogNetworkPlatformFile, Display, "Received '%d' cached file(s) from server", ServerCachedFiles.Num());

	bool bDeleteAllFiles = true;
	// Check the stored cooked version
	FString CookedVersionFile = FPaths::GeneratedConfigDir() / TEXT("CookedVersion.txt");

	if (InnerPlatformFile->FileExists(*CookedVersionFile) == true)
	{
		TUniquePtr<IFileHandle> FileHandle(InnerPlatformFile->OpenRead(*CookedVersionFile));
		if (FileHandle.IsValid())
		{
			FPackageFileVersion StoredPackageCookedVersion;
			int32 StoredPackageCookedLicenseeVersion;
			if (FileHandle->Read((uint8*)&StoredPackageCookedVersion, sizeof(FPackageFileVersion)) == true)
			{
				if (FileHandle->Read((uint8*)&StoredPackageCookedLicenseeVersion, sizeof(int32)) == true)
				{
					if ((ServerPackageVersion == StoredPackageCookedVersion) &&
						(ServerPackageLicenseeVersion == StoredPackageCookedLicenseeVersion))
					{
						bDeleteAllFiles = false;
					}
					else
					{
						UE_LOGF(LogNetworkPlatformFile, Display,
							"Engine version mismatch: Server %d.%d, Stored %d.%d\n",
							ServerPackageVersion.ToValue(), ServerPackageLicenseeVersion,
							StoredPackageCookedVersion.ToValue(), StoredPackageCookedLicenseeVersion);
					}
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogNetworkPlatformFile, Display, "Cooked version file missing: %ls\n", *CookedVersionFile);
	}

	TMap<FString, FDateTime> ClientLocalFiles;
	{
		TArray<FString> DirectoriesToSkip;
		TArray<FString> DirectoriesToNotRecurse;
		FLocalTimestampDirectoryVisitor Visitor(*InnerPlatformFile, DirectoriesToSkip, DirectoriesToNotRecurse, false, /* bMakeLowerCased */ true);

		for (TArray<FString>::TConstIterator RootPathIt(ServerRootContentDirectories); RootPathIt; ++RootPathIt)
		{
			const FString& ContentFolder = *RootPathIt;
			InnerPlatformFile->IterateDirectory(*ContentFolder, Visitor);
		}

		ClientLocalFiles = MoveTemp(Visitor.FileTimes);
	}

	UE_LOGF(LogNetworkPlatformFile, Display, "Found '%d' locally cached file(s)", ClientLocalFiles.Num());

	if (FParse::Param(FCommandLine::Get(), TEXT("DeleteLocalCache")))
	{
		bDeleteAllFiles = true;
	}

	if (bDeleteAllFiles)
	{
		UE_LOGF(LogNetworkPlatformFile, Display, "Deleting all '%d' cached file(s) due to missing cooked version file...", ClientLocalFiles.Num());

		// Make sure the config file exists...
		InnerPlatformFile->CreateDirectoryTree(*(FPaths::GeneratedConfigDir()));
		// Update the cooked version file
		TUniquePtr<IFileHandle> FileHandle(InnerPlatformFile->OpenWrite(*CookedVersionFile));
		if (FileHandle.IsValid())
		{
			FileHandle->Write((const uint8*)&ServerPackageVersion, sizeof(FPackageFileVersion));
			FileHandle->Write((const uint8*)&ServerPackageLicenseeVersion, sizeof(int32));
		}
		else
		{
			UE_LOGF(LogNetworkPlatformFile, Warning, "Failed to write cooked version file '%ls'", *CookedVersionFile);
		}
	}

	int32 NumDeleted = 0;
	for (const auto& ClientFileTimePair : ClientLocalFiles)
	{
		const FString& Filename = ClientFileTimePair.Key;
		bool bDeleteFile = bDeleteAllFiles;

		if (bDeleteFile)
		{
			UE_LOGF(LogNetworkPlatformFile, Display, "Deleting cached file '%ls'", *Filename);
		}
		else
		{
			if (const FDateTime* ServerTime = ServerCachedFiles.Find(Filename))
			{
				FDateTime ClientTime = ClientFileTimePair.Value;
				FTimespan TimeDiff = *ServerTime - ClientTime;
				double TimeDiffInSeconds = TimeDiff.GetTotalSeconds();
				bDeleteFile = (TimeDiffInSeconds > 1.0) || (TimeDiffInSeconds < -1.0);

				if (bDeleteFile)
				{
					UE_LOGF(LogNetworkPlatformFile, Display, "Deleting outdated cached file '%ls' with time difference '%.3f's", *Filename, TimeDiffInSeconds);
				}
				else
				{
					UE_LOGF(LogNetworkPlatformFile, Display, "Keeping cached file '%ls' with time difference '%.3f's", *Filename, TimeDiffInSeconds);
				}
			}
			else
			{
				UE_LOGF(LogNetworkPlatformFile, Display, "Deleting cached file '%ls' missing on server", *Filename);
				bDeleteFile = true;
			}
		}

		if (bDeleteFile)
		{
			// ignore pak files they won't be mounted anyway 
			if (FCString::Stricmp(*FPaths::GetExtension(Filename), TEXT("pak")) != 0)
			{
				InnerPlatformFile->DeleteFile(*Filename);
				++NumDeleted;
			}
		}
	}

	UE_LOGF(LogNetworkPlatformFile, Display, "Deleted '%d' of '%d' cached file(s)", NumDeleted, ClientLocalFiles.Num());
}

FNetworkPlatformFile::~FNetworkPlatformFile()
{
	if (!IsEngineExitRequested()) // the socket subsystem is probably already gone, so it will crash if we clean up
	{
		FScopeLock ScopeLock(&SynchronizationObject);
		if ( FinishedAsyncNetworkReadUnsolicitedFiles )
		{
			delete FinishedAsyncNetworkReadUnsolicitedFiles; // wait here for any async unsolicited files to finish reading being read from the network 
			FinishedAsyncNetworkReadUnsolicitedFiles = NULL;
		}
		if ( FinishedAsyncWriteUnsolicitedFiles )
		{
			delete FinishedAsyncWriteUnsolicitedFiles; // wait here for any async unsolicited files to finish writing 
			FinishedAsyncWriteUnsolicitedFiles = NULL;
		}
		
		Connection.Reset(); // close our sockets.
	}
}

bool FNetworkPlatformFile::DeleteFile(const TCHAR* Filename)
{
//	FScopeLock ScopeLock(&SynchronizationObject);

	// make and send payload (this is how we would do for sending all commands over the network)
//	FNetworkFileArchive Payload(NFS_Messages::DeleteFile);
//	Payload << Filename;
//	return FNFSMessageHeader::WrapAndSendPayload(Payload, FileSocket);

	// perform a local operation
	return InnerPlatformFile->DeleteFile(Filename);
}

bool FNetworkPlatformFile::MoveFile(const TCHAR* To, const TCHAR* From)
{
//	FScopeLock ScopeLock(&SynchronizationObject);

	// make and send payload (this is how we would do for sending all commands over the network)
//	FNetworkFileArchive Payload(NFS_Messages::MoveFile);
//	Payload << To << From;
//	return FNFSMessageHeader::WrapAndSendPayload(Payload, FileSocket);

	FString RelativeFrom = From;
	MakeStandardNetworkFilename(RelativeFrom);

	// don't copy files in local directories
	if (!IsInLocalDirectory(RelativeFrom))
	{
		// make sure the source file exists here
		EnsureFileIsLocal(RelativeFrom);
	}

	// perform a local operation
	return InnerPlatformFile->MoveFile(To, From);
}

bool FNetworkPlatformFile::SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue)
{
//	FScopeLock ScopeLock(&SynchronizationObject);

	// perform a local operation
	return InnerPlatformFile->SetReadOnly(Filename, bNewReadOnlyValue);
}

void FNetworkPlatformFile::SetTimeStamp(const TCHAR* Filename, FDateTime DateTime)
{
	// perform a local operation
	InnerPlatformFile->SetTimeStamp(Filename, DateTime);
}

IFileHandle* FNetworkPlatformFile::OpenRead(const TCHAR* Filename, bool bAllowWrite)
{
//	FScopeLock ScopeLock(&SynchronizationObject);

	FString RelativeFilename = Filename;
	MakeStandardNetworkFilename(RelativeFilename);
	// don't copy files in local directories
	if (!IsInLocalDirectory(RelativeFilename))
	{
		EnsureFileIsLocal(RelativeFilename);
	}

	double StartTime;
	float ThisTime;

	StartTime = FPlatformTime::Seconds();
	IFileHandle* Result = InnerPlatformFile->OpenRead(Filename, bAllowWrite);

	ThisTime = 1000.0f * float(FPlatformTime::Seconds() - StartTime);
	//UE_LOGF(LogNetworkPlatformFile, Display, "Open local file %6.2fms", ThisTime);
	return Result;
}

IFileHandle* FNetworkPlatformFile::OpenWrite(const TCHAR* Filename, bool bAppend, bool bAllowRead)
{
//	FScopeLock ScopeLock(&SynchronizationObject);

	// just let the physical file interface write the file (we don't write over the network)
	return InnerPlatformFile->OpenWrite(Filename, bAppend, bAllowRead);
}

bool FNetworkPlatformFile::CreateDirectoryTree(const TCHAR* Directory)
{
	//	FScopeLock ScopeLock(&SynchronizationObject);

	// perform a local operation
	return InnerPlatformFile->CreateDirectoryTree(Directory);
}

bool FNetworkPlatformFile::CreateDirectory(const TCHAR* Directory)
{
//	FScopeLock ScopeLock(&SynchronizationObject);

	// perform a local operation
	return InnerPlatformFile->CreateDirectory(Directory);
}

bool FNetworkPlatformFile::DeleteDirectory(const TCHAR* Directory)
{
//	FScopeLock ScopeLock(&SynchronizationObject);

	// perform a local operation
	return InnerPlatformFile->DeleteDirectory(Directory);
}

FFileStatData FNetworkPlatformFile::GetStatData(const TCHAR* FilenameOrDirectory)
{
//	FScopeLock ScopeLock(&SynchronizationObject);

	// perform a local operation
	return InnerPlatformFile->GetStatData(FilenameOrDirectory);
}

bool FNetworkPlatformFile::IterateDirectory(const TCHAR* InDirectory, IPlatformFile::FDirectoryVisitor& Visitor)
{
//	FScopeLock ScopeLock(&SynchronizationObject);

	// for .dll, etc searches that don't specify a path, we need to strip off the path
	// before we send it to the visitor
	bool bHadNoPath = InDirectory[0] == 0;

	// local files go right to the source
	FString RelativeDirectory = InDirectory;
	MakeStandardNetworkFilename(RelativeDirectory);
	if (IsInLocalDirectory(RelativeDirectory))
	{
		return InnerPlatformFile->IterateDirectory(InDirectory, Visitor);
	}

	// we loop until this is false
	bool RetVal = true;

	FServerTOC::FDirectory* ServerDirectory = ServerFiles.FindDirectory(RelativeDirectory);
	if (ServerDirectory != NULL)
	{
		// loop over the server files and look if they are in this exact directory
		for (FServerTOC::FDirectory::TIterator It(*ServerDirectory); It && RetVal == true; ++It)
		{
			if (FPaths::GetPath(It.Key()) == RelativeDirectory)
			{
				// timestamps of 0 mean directories
				bool bIsDirectory = It.Value() == 0;
			
				// visit (stripping off the path if needed)
				RetVal = Visitor.CallShouldVisitAndVisit(bHadNoPath ? *FPaths::GetCleanFilename(It.Key()) : *It.Key(), bIsDirectory);
			}
		}
	}

	return RetVal;
}

bool FNetworkPlatformFile::IterateDirectoryRecursively(const TCHAR* InDirectory, IPlatformFile::FDirectoryVisitor& Visitor)
{
//	FScopeLock ScopeLock(&SynchronizationObject);

	// local files go right to the source
	FString RelativeDirectory = InDirectory;
	MakeStandardNetworkFilename(RelativeDirectory);

	if (IsInLocalDirectory(RelativeDirectory))
	{
		return InnerPlatformFile->IterateDirectoryRecursively(InDirectory, Visitor);
	}

	// we loop until this is false
	bool RetVal = true;

	for (TMap<FString, FServerTOC::FDirectory*>::TIterator DirIt(ServerFiles.Directories); DirIt && RetVal == true; ++DirIt)
	{
		if (DirIt.Key().StartsWith(RelativeDirectory))
		{
			FServerTOC::FDirectory& ServerDirectory = *DirIt.Value();
		
			// loop over the server files and look if they are in this exact directory
			for (FServerTOC::FDirectory::TIterator It(ServerDirectory); It && RetVal == true; ++It)
			{
				// timestamps of 0 mean directories
				bool bIsDirectory = It.Value() == 0;

				// visit!
				RetVal = Visitor.CallShouldVisitAndVisit(*It.Key(), bIsDirectory);
			}
		}
	}

	return RetVal;
}

bool FNetworkPlatformFile::IterateDirectoryStat(const TCHAR* InDirectory, IPlatformFile::FDirectoryStatVisitor& Visitor)
{
//	FScopeLock ScopeLock(&SynchronizationObject);

	// for .dll, etc searches that don't specify a path, we need to strip off the path
	// before we send it to the visitor
	bool bHadNoPath = InDirectory[0] == 0;

	// local files go right to the source
	FString RelativeDirectory = InDirectory;
	MakeStandardNetworkFilename(RelativeDirectory);
	if (IsInLocalDirectory(RelativeDirectory))
	{
		return InnerPlatformFile->IterateDirectoryStat(InDirectory, Visitor);
	}

	// we loop until this is false
	bool RetVal = true;

	FServerTOC::FDirectory* ServerDirectory = ServerFiles.FindDirectory(RelativeDirectory);
	if (ServerDirectory != NULL)
	{
		// loop over the server files and look if they are in this exact directory
		for (FServerTOC::FDirectory::TIterator It(*ServerDirectory); It && RetVal == true; ++It)
		{
			if (FPaths::GetPath(It.Key()) == RelativeDirectory)
			{
				// timestamps of 0 mean directories
				bool bIsDirectory = It.Value() == 0;
			
				// todo: this data is just wrong for most things, but can we afford to get the files from the server to get the correct info? Could the server provide this instead?
				const FFileStatData StatData(
					FDateTime::MinValue(), 
					FDateTime::MinValue(), 
					(bIsDirectory) ? FDateTime::MinValue() : It.Value(),
					-1,		// FileSize
					bIsDirectory, 
					true	// IsReadOnly
					);

				// visit (stripping off the path if needed)
				RetVal = Visitor.CallShouldVisitAndVisit(bHadNoPath ? *FPaths::GetCleanFilename(It.Key()) : *It.Key(), StatData);
			}
		}
	}

	return RetVal;
}

bool FNetworkPlatformFile::IterateDirectoryStatRecursively(const TCHAR* InDirectory, IPlatformFile::FDirectoryStatVisitor& Visitor)
{
//	FScopeLock ScopeLock(&SynchronizationObject);

	// local files go right to the source
	FString RelativeDirectory = InDirectory;
	MakeStandardNetworkFilename(RelativeDirectory);

	if (IsInLocalDirectory(RelativeDirectory))
	{
		return InnerPlatformFile->IterateDirectoryStatRecursively(InDirectory, Visitor);
	}

	// we loop until this is false
	bool RetVal = true;

	for (TMap<FString, FServerTOC::FDirectory*>::TIterator DirIt(ServerFiles.Directories); DirIt && RetVal == true; ++DirIt)
	{
		if (DirIt.Key().StartsWith(RelativeDirectory))
		{
			FServerTOC::FDirectory& ServerDirectory = *DirIt.Value();
		
			// loop over the server files and look if they are in this exact directory
			for (FServerTOC::FDirectory::TIterator It(ServerDirectory); It && RetVal == true; ++It)
			{
				// timestamps of 0 mean directories
				bool bIsDirectory = It.Value() == 0;

				// todo: this data is just wrong for most things, but can we afford to get the files from the server to get the correct info? Could the server provide this instead?
				const FFileStatData StatData(
					FDateTime::MinValue(), 
					FDateTime::MinValue(), 
					(bIsDirectory) ? FDateTime::MinValue() : It.Value(),
					0,		// FileSize
					bIsDirectory, 
					true	// IsReadOnly
					);

				// visit!
				RetVal = Visitor.CallShouldVisitAndVisit(*It.Key(), StatData);
			}
		}
	}

	return RetVal;
}

bool FNetworkPlatformFile::DeleteDirectoryRecursively(const TCHAR* Directory)
{
//	FScopeLock ScopeLock(&SynchronizationObject);

	// perform a local operation
	return InnerPlatformFile->DeleteDirectory(Directory);
}

bool FNetworkPlatformFile::CopyFile(const TCHAR* To, const TCHAR* From, EPlatformFileRead ReadFlags, EPlatformFileWrite WriteFlags)
{
//	FScopeLock ScopeLock(&SynchronizationObject);

	FString RelativeFrom = From;
	MakeStandardNetworkFilename(RelativeFrom);

	// don't copy files in local directories
	if (!IsInLocalDirectory(RelativeFrom))
	{
		// make sure the source file exists here
		EnsureFileIsLocal(RelativeFrom);
	}

	// perform a local operation
	return InnerPlatformFile->CopyFile(To, From, ReadFlags, WriteFlags);
}

FString FNetworkPlatformFile::ConvertToAbsolutePathForExternalAppForRead( const TCHAR* Filename )
{
	FString RelativeFrom = Filename;
	MakeStandardNetworkFilename(RelativeFrom);
	
	if (!IsInLocalDirectory(RelativeFrom))
	{
		EnsureFileIsLocal(RelativeFrom);
	}
	return InnerPlatformFile->ConvertToAbsolutePathForExternalAppForRead(Filename);
}

FString FNetworkPlatformFile::ConvertToAbsolutePathForExternalAppForWrite( const TCHAR* Filename )
{
	FString RelativeFrom = Filename;
	MakeStandardNetworkFilename(RelativeFrom);

	if (!IsInLocalDirectory(RelativeFrom))
	{
		EnsureFileIsLocal(RelativeFrom);
	}
	return InnerPlatformFile->ConvertToAbsolutePathForExternalAppForWrite(Filename);
}

bool FNetworkPlatformFile::DirectoryExists(const TCHAR* Directory)
{
	if (InnerPlatformFile->DirectoryExists(Directory))
	{
		return true;
	}
	// If there are any syncable files in this directory, consider it existing
	FString RelativeDirectory = Directory;
	MakeStandardNetworkFilename(RelativeDirectory);		

	FServerTOC::FDirectory* ServerDirectory = ServerFiles.FindDirectory(RelativeDirectory);
	return ServerDirectory != NULL;
}

void FNetworkPlatformFile::GetFileInfo(const TCHAR* Filename, FFileInfo& Info)
{
	FString RelativeFilename = Filename;
	MakeStandardNetworkFilename(RelativeFilename);

	// don't copy files in local directories
	if (!IsInLocalDirectory(RelativeFilename))
	{
		EnsureFileIsLocal(RelativeFilename);
	}

	const FFileStatData StatData = InnerPlatformFile->GetStatData(Filename);
	Info.FileExists = StatData.bIsValid && !StatData.bIsDirectory;
	Info.ReadOnly = StatData.bIsReadOnly;
	Info.Size = StatData.FileSize;
	Info.TimeStamp = StatData.ModificationTime;
	Info.AccessTimeStamp = StatData.AccessTime;
}

void FNetworkPlatformFile::ConvertServerFilenameToClientFilename(FString& FilenameToConvert)
{
	FNetworkPlatformFile::ConvertServerFilenameToClientFilename(FilenameToConvert, ServerEngineDir, ServerProjectDir, ServerEnginePlatformExtensionsDir, ServerProjectPlatformExtensionsDir);
}

void FNetworkPlatformFile::FillGetFileList(FNetworkFileArchive& Payload)
{
	TArray<FString> TargetPlatformNames;
	FPlatformMisc::GetValidTargetPlatforms(TargetPlatformNames);
	FString GameName = FApp::GetProjectName();
	if (FPaths::IsProjectFilePathSet())
	{
		GameName = FPaths::GetProjectFilePath();
	}

	FString EngineRelPath = FPaths::EngineDir();
	FString EngineRelPluginPath = FPaths::EnginePluginsDir();
	FString GameRelPath = FPaths::ProjectDir();
	FString GameRelPluginPath = FPaths::ProjectPluginsDir();
	FString EnginePlatformExtensionsDir = UnsafeEnginePlatformExtensionDir();
	FString ProjectPlatformExtensionsDir = UnsafeProjectPlatformExtensionDir();

	TArray<FString> Directories;
	Directories.Add(EngineRelPath);
	Directories.Add(GameRelPath);

	Payload << TargetPlatformNames;
	Payload << GameName;
	Payload << EngineRelPath;
	Payload << GameRelPath;
	Payload << EnginePlatformExtensionsDir;
	Payload << ProjectPlatformExtensionsDir;
	Payload << EngineRelPluginPath;
	Payload << GameRelPluginPath;
	Payload << Directories;
	Payload << ConnectionFlags;

	FString VersionInfo = GetVersionInfo();
	Payload << VersionInfo;

	bool bBindAllDummy;
	TSharedRef<FInternetAddr> HostAddr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalHostAddr(*GLog, bBindAllDummy);
	FString HostAddress = HostAddr->ToString(false);
	Payload << HostAddress;

	TMap<FString,FString> CustomPlatformData;
	FPlatformMisc::GetNetworkFileCustomData(CustomPlatformData);
	Payload << CustomPlatformData;
}

void FNetworkPlatformFile::ProcessServerInitialResponse(FArrayReader& InResponse, FPackageFileVersion& OutServerPackageVersion, int32& OutServerPackageLicenseeVersion)
{
	// Receive the cooked version information.
	InResponse << OutServerPackageVersion;
	InResponse << OutServerPackageLicenseeVersion;

	// receive the server engine and game dir
	InResponse << ServerEngineDir;
	InResponse << ServerProjectDir;
	InResponse << ServerEnginePlatformExtensionsDir;
	InResponse << ServerProjectPlatformExtensionsDir;

	UE_LOGF(LogNetworkPlatformFile, Display, "    Server EngineDir      = %ls", *ServerEngineDir);
	UE_LOGF(LogNetworkPlatformFile, Display, "     Local EngineDir      = %ls", *FPaths::EngineDir());
	UE_LOGF(LogNetworkPlatformFile, Display, "    Server ProjectDir     = %ls", *ServerProjectDir);
	UE_LOGF(LogNetworkPlatformFile, Display, "     Local ProjectDir     = %ls", *FPaths::ProjectDir());
	UE_LOGF(LogNetworkPlatformFile, Display, "    Server EnginePlatformExtDir = %ls", *ServerEnginePlatformExtensionsDir);
	UE_LOGF(LogNetworkPlatformFile, Display, "     Local EnginePlatformExtDir = %ls", *UnsafeEnginePlatformExtensionDir());
	UE_LOGF(LogNetworkPlatformFile, Display, "    Server ProjectPlatformExtDir = %ls", *ServerProjectPlatformExtensionsDir);
	UE_LOGF(LogNetworkPlatformFile, Display, "     Local ProjectPlatformExtDir = %ls", *UnsafeProjectPlatformExtensionDir());

	// Receive a list of files and their timestamps.
	TMap<FString, FDateTime> ServerFileMap;
	InResponse << ServerFileMap;
	for (TMap<FString, FDateTime>::TIterator It(ServerFileMap); It; ++It)
	{
		FString ServerFile = It.Key();
		ConvertServerFilenameToClientFilename(ServerFile);
		ServerFiles.AddFileOrDirectory(ServerFile, It.Value());
	}
}


FString FNetworkPlatformFile::GetVersionInfo() const
{
	return FString();
}

bool FNetworkPlatformFile::SendReadMessage(uint8* Destination, int64 BytesToRead)
{
//	FScopeLock ScopeLock(&SynchronizationObject);

	return true;
}

bool FNetworkPlatformFile::SendWriteMessage(const uint8* Source, int64 BytesToWrite)
{
//	FScopeLock ScopeLock(&SynchronizationObject);

	return true;
}

bool FNetworkPlatformFile::SendMessageToServer(const TCHAR* Message, IPlatformFile::IFileServerMessageHandler* Handler)
{
#if WITH_COTF
	if (!Connection->IsConnected())
	{
		return false;
	}
	if (FCString::Stricmp(Message, TEXT("RecompileShaders")) == 0)
	{
		UE::Cook::FCookOnTheFlyRequest Request(UE::Cook::ECookOnTheFlyMessage::RecompileShaders);
		{
			TUniquePtr<FArchive> Ar = Request.WriteBody();
			Handler->FillPayload(*Ar);
		}

		UE::Cook::FCookOnTheFlyResponse Response = Connection->SendRequest(Request).Get();
		if (Response.IsOk())
		{
			TUniquePtr<FArchive> Ar = Response.ReadBody();
			Handler->ProcessResponse(*Ar);
		}

		return Response.IsOk();
	}
#endif
	return false;
}


static FThreadSafeCounter OutstandingAsyncWrites;

class FAsyncNetworkWriteWorker : public FNonAbandonableTask
{
public:
	/** Filename To write to**/
	FString							Filename;
	/** An archive to read the file contents from */
	FArrayReader* FileArchive;
	TUniquePtr<FArrayReader> FileArchiveOwner;

	/** timestamp for the file **/
	FDateTime ServerTimeStamp;
	IPlatformFile& InnerPlatformFile;
	FScopedEvent* Event;

	FAsyncNetworkWriteWorker(const TCHAR* InFilename, FArrayReader& InArchive, FDateTime InServerTimeStamp, IPlatformFile& InInnerPlatformFile)
		: Filename(InFilename)
		, FileArchive(&InArchive)
		, ServerTimeStamp(InServerTimeStamp)
		, InnerPlatformFile(InInnerPlatformFile)
		, Event(nullptr)
	{
	}

	FAsyncNetworkWriteWorker(const TCHAR* InFilename, TUniquePtr<FArrayReader> InArchive, FDateTime InServerTimeStamp, IPlatformFile& InInnerPlatformFile, FScopedEvent* InEvent)
		: Filename(InFilename)
		, FileArchive(InArchive.Get())
		, FileArchiveOwner(MoveTemp(InArchive))
		, ServerTimeStamp(InServerTimeStamp)
		, InnerPlatformFile(InInnerPlatformFile)
		, Event(InEvent)
	{
	}
		
	/** Write the file  */
	void DoWork()
	{
		// Read FileSize first so that the correct amount of data is read from the archive
		// before exiting this worker.
		uint64 FileSize = 0;
		*FileArchive << FileSize;

		bool bCopiedExternally = (FileSize == MAX_uint64); // -1 filesize means that we already copied it via TargetPlatform

		if (!bCopiedExternally && InnerPlatformFile.FileExists(*Filename) ) 
		{
			InnerPlatformFile.SetReadOnly(*Filename, false);
			InnerPlatformFile.DeleteFile(*Filename);
		}

		if (ServerTimeStamp != FDateTime::MinValue())  // if the file didn't actually exist on the server, don't create a zero byte file
		{
			if (!bCopiedExternally)
			{
				FString TempFilename = Filename + TEXT(".tmp");
				InnerPlatformFile.CreateDirectoryTree(*FPaths::GetPath(Filename));
				{
					TUniquePtr<IFileHandle> FileHandle;
					FileHandle.Reset(InnerPlatformFile.OpenWrite(*TempFilename));

					if (!FileHandle)
					{
						UE_LOGF(LogNetworkPlatformFile, Fatal, "Could not open file for writing '%ls'.", *TempFilename);
					}

					// now write the file from bytes pulled from the archive
					// read/write a chunk at a time
					uint64 RemainingData = FileSize;
					if (!FileHandle->Write(FileArchive->GetData() + FileArchive->Tell(), FileSize))
					{
						UE_LOGF(LogNetworkPlatformFile, Fatal, "Could not write '%ls'.", *TempFilename);
					}
					FileArchive->Seek(FileArchive->Tell() + FileSize);

					// delete async write archives
					if (Event)
					{
						FileArchive = nullptr;
						FileArchiveOwner.Reset();
					}

					if (InnerPlatformFile.FileSize(*TempFilename) != FileSize)
					{
						UE_LOGF(LogNetworkPlatformFile, Fatal, "Did not write '%ls'.", *TempFilename);
					}
				}

				// rename from temp filename to real filename
				InnerPlatformFile.MoveFile(*Filename, *TempFilename);

				// now set the server's timestamp on the local file (so we can make valid comparisons)
				InnerPlatformFile.SetTimeStamp(*Filename, ServerTimeStamp);

			    FDateTime CheckTime = InnerPlatformFile.GetTimeStamp(*Filename);
			    if (CheckTime < ServerTimeStamp)
			    {
				    UE_LOGF(LogNetworkPlatformFile, Fatal, "Could Not Set Timestamp '%ls'  %ls < %ls.", *Filename, *CheckTime.ToString(), *ServerTimeStamp.ToString());
			    }
			}

		}
		if (Event)
		{
			if (OutstandingAsyncWrites.Decrement() == 0)
			{
				Event->Trigger(); // last file, fire trigger
			}
		}
	}

	FORCEINLINE TStatId GetStatId() const
	{
		return TStatId();
		//RETURN_QUICK_DECLARE_CYCLE_STAT(FAsyncNetworkWriteWorker, STATGROUP_ThreadPoolAsyncTasks);
	}
};

/**
 * Write a file async or sync, with the data coming from a FArrayReader
 */
static void SyncWriteFile(FArrayReader* Archive, const FString& Filename, FDateTime ServerTimeStamp, IPlatformFile& InnerPlatformFile)
{
	(new FAutoDeleteAsyncTask<FAsyncNetworkWriteWorker>(*Filename, *Archive, ServerTimeStamp, InnerPlatformFile))->StartSynchronousTask();
}

static void AsyncWriteFile(TUniquePtr<FArrayReader> Archive, const FString& Filename, FDateTime ServerTimeStamp, IPlatformFile& InnerPlatformFile, FScopedEvent* Event)
{
	(new FAutoDeleteAsyncTask<FAsyncNetworkWriteWorker>(*Filename, MoveTemp(Archive), ServerTimeStamp, InnerPlatformFile, Event))->StartBackgroundTask();
}

void AsyncReadUnsolicitedFiles(int32 InNumUnsolictedFiles, FNetworkPlatformFile& InNetworkFile, IPlatformFile& InInnerPlatformFile, FString& InServerEngineDir, FString& InServerProjectDir, FString& InServerEnginePlatformExtensionsDir, FString& InServerProjectPlatformExtensionsDir, FScopedEvent *InNetworkDoneEvent, FScopedEvent *InWritingDoneEvent)
{
	class FAsyncReadUnsolicitedFile : public FNonAbandonableTask
	{
	public:
		int32 NumUnsolictedFiles;
		FNetworkPlatformFile& NetworkFile;
		IPlatformFile& InnerPlatformFile;
		FString ServerEngineDir;
		FString ServerProjectDir;
		FString ServerEnginePlatformExtensionsDir;
		FString ServerProjectPlatformExtensionsDir;
		FScopedEvent* NetworkDoneEvent; // finished using the network
		FScopedEvent* WritingDoneEvent; // finished writing the files to disk

		FAsyncReadUnsolicitedFile(int32 In_NumUnsolictedFiles, FNetworkPlatformFile* In_NetworkFile, IPlatformFile* In_InnerPlatformFile, FString& In_ServerEngineDir, FString& In_ServerProjectDir, FString In_ServerEnginePlatformExtensionsDir, FString In_ServerProjectPlatformExtensionsDir, FScopedEvent *In_NetworkDoneEvent, FScopedEvent *In_WritingDoneEvent)
			: NumUnsolictedFiles(In_NumUnsolictedFiles)
			, NetworkFile(*In_NetworkFile)
			, InnerPlatformFile(*In_InnerPlatformFile)
			, ServerEngineDir(In_ServerEngineDir)
			, ServerProjectDir(In_ServerProjectDir)
			, ServerEnginePlatformExtensionsDir(In_ServerEnginePlatformExtensionsDir)
			, ServerProjectPlatformExtensionsDir(In_ServerProjectPlatformExtensionsDir)
			, NetworkDoneEvent(In_NetworkDoneEvent)
			, WritingDoneEvent(In_WritingDoneEvent)
		{
		}
		
		/** Write the file  */
		void DoWork()
		{
			OutstandingAsyncWrites.Add( NumUnsolictedFiles );
			for (int32 Index = 0; Index < NumUnsolictedFiles; Index++)
			{
				TUniquePtr<FArrayReader> UnsolictedResponse(new FArrayReader);
				if (!NetworkFile.ReceiveResponse(*UnsolictedResponse))
				{
					UE_LOGF(LogNetworkPlatformFile, Fatal, "Receive failure!");
					return;
				}
				FString UnsolictedReplyFile;
				*UnsolictedResponse << UnsolictedReplyFile;

				if (!UnsolictedReplyFile.IsEmpty())
				{
					FNetworkPlatformFile::ConvertServerFilenameToClientFilename(UnsolictedReplyFile, ServerEngineDir, ServerProjectDir, ServerEnginePlatformExtensionsDir, ServerProjectPlatformExtensionsDir);
					// get the server file timestamp
					FDateTime UnsolictedServerTimeStamp;
					*UnsolictedResponse << UnsolictedServerTimeStamp;

					// write the file by pulling out of the FArrayReader
					AsyncWriteFile(MoveTemp(UnsolictedResponse), UnsolictedReplyFile, UnsolictedServerTimeStamp, InnerPlatformFile, WritingDoneEvent);
				}
			}
			NetworkDoneEvent->Trigger();
		}

		FORCEINLINE TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FAsyncReadUnsolicitedFile, STATGROUP_ThreadPoolAsyncTasks);
		}
	};

	(new FAutoDeleteAsyncTask<FAsyncReadUnsolicitedFile>(InNumUnsolictedFiles, &InNetworkFile, &InInnerPlatformFile, InServerEngineDir, InServerProjectDir, InServerEnginePlatformExtensionsDir, InServerProjectPlatformExtensionsDir, InNetworkDoneEvent, InWritingDoneEvent))->StartSynchronousTask();
}

bool FNetworkPlatformFile::IsMediaExtension(const TCHAR* Ext)
{
	if (*Ext != TEXT('.'))
	{
		return MP4Extension.EndsWith(Ext);
	}
	else
	{
		return MP4Extension == Ext;
	}
}

bool FNetworkPlatformFile::IsAdditionalCookedFileExtension(const TCHAR* Ext)
{
	if (*Ext != TEXT('.'))
	{
		return BulkFileExtension.EndsWith(Ext) || FontFileExtension.EndsWith(Ext) || ExpFileExtension.EndsWith(Ext);
	}
	else
	{
		return BulkFileExtension == Ext || FontFileExtension == Ext || ExpFileExtension == Ext;
	}
}

/**
 * Given a filename, make sure the file exists on the local filesystem
 */

void FNetworkPlatformFile::EnsureFileIsLocal(const FString& Filename)
{
	double StartTime;
	float ThisTime;
	StartTime = FPlatformTime::Seconds();

	UE_LOGF(LogNetworkPlatformFile, Verbose, "Searching for %ls locally ", *Filename);

	{
		FScopeLock ScopeLock(&SynchronizationObject);
		// have we already cached this file? 
		if (CachedLocalFiles.Find(Filename) != NULL)
		{
			return;
		}

		bool bIncrementedPackageWaits = false;
		if (FinishedAsyncNetworkReadUnsolicitedFiles)
		{
			if (FinishedAsyncNetworkReadUnsolicitedFiles->Get() == 0)
			{
				++UnsolicitedPackageWaits;
				bIncrementedPackageWaits = true;
			}
			delete FinishedAsyncNetworkReadUnsolicitedFiles; // wait here for any async unsolicited files to finish reading being read from the network 
			FinishedAsyncNetworkReadUnsolicitedFiles = NULL;
		}
		if (FinishedAsyncWriteUnsolicitedFiles)
		{
			if (bIncrementedPackageWaits == false && FinishedAsyncWriteUnsolicitedFiles->Get() == 0)
			{
				++UnsolicitedPackageWaits;
			}
			delete FinishedAsyncWriteUnsolicitedFiles; // wait here for any async unsolicited files to finish writing to disk
			FinishedAsyncWriteUnsolicitedFiles = NULL;
		}
	}

	FScopeLock ScopeLock(&SynchronizationObject);
	ThisTime = 1000.0f * float(FPlatformTime::Seconds() - StartTime);
	TotalWaitForAsyncUnsolicitedPackages += ThisTime;
	//UE_LOGF(LogNetworkPlatformFile, Display, "Lock and wait for old async writes %6.2fms", ThisTime);

	if (CachedLocalFiles.Find(Filename) != NULL)
	{
		++UnsolicitedPackagesHits;
		return;
	}

	UE_LOGF(LogNetworkPlatformFile, Verbose, "Attempting to get %ls from server", *Filename);

	// even if an error occurs later, we still want to remember not to try again
	CachedLocalFiles.Add(Filename);
	StartTime = FPlatformTime::Seconds();

	// no need to read it if it already exists 
	// @todo: Handshake with server to delete files that are out of date
	if (InnerPlatformFile->FileExists(*Filename))
	{
		++TotalFilesFoundLocally;
		UE_LOGF(LogNetworkPlatformFile, Verbose, "File %ls exists locally but wasn't in cache", *Filename);
		return;
	}

	++TotalFilesSynced;

	

	ThisTime = 1000.0f * float(FPlatformTime::Seconds() - StartTime);
	//UE_LOGF(LogNetworkPlatformFile, Display, "Check for local file %6.2fms - %ls", ThisTime, *Filename);

	// this is a bit of a waste if we aren't doing cook on the fly, but we assume missing asset files are relatively rare
	FString Extension = FPaths::GetExtension(Filename, true);
	bool bIsCookable = GConfig && GConfig->IsReadyForUse() && (FPackageName::IsPackageExtension(*Extension) || IsMediaExtension(*Extension) || IsAdditionalCookedFileExtension(*Extension));

	// we only copy files that actually exist on the server, can greatly reduce network traffic for, say,
	// the INT file each package tries to load
	if (!bIsCookable && (ServerFiles.FindFile(Filename) == NULL))
	{
		// Uncomment this to have the server file list dumped
		// the first time a file requested is not found.

		UE_LOGF(LogNetworkPlatformFile, Verbose, "Didn't find %ls in server files list", *Filename);

#if 0
		static bool sb_DumpedServer = false;
		if (sb_DumpedServer == false)
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Dumping server files... %s not found\n"), *Filename);
			for (TMap<FString, FServerTOC::FDirectory*>::TIterator ServerDumpIt(ServerFiles.Directories); ServerDumpIt; ++ServerDumpIt)
			{
				FServerTOC::FDirectory& Directory = *ServerDumpIt.Value();
				for (FServerTOC::FDirectory::TIterator DirDumpIt(Directory); DirDumpIt; ++DirDumpIt)
				{
					FPlatformMisc::LowLevelOutputDebugStringf(TEXT("%10s - %s\n"), *(DirDumpIt.Value().ToString()), *(DirDumpIt.Key()));
				}
			}
			sb_DumpedServer = true;
		}
#endif
		return;
	}

	if (bIsCookable && (FPackageName::GetAssetPackageExtension() == Extension))
	{
		// This is meant to handle the fact that FindPackageFileWithoutExtension will attempt to load a *.umap file first as a *.uasset file
		// Instead of needlessly asking the COTF server to load and cook the map again, this will just see the *.umap in the local cache, and
		// handle the request for the *.uasset version of the file as being not present.
		FString AlternatePackageName = FPaths::ChangeExtension(Filename, FPackageName::GetMapPackageExtension());
		if (InnerPlatformFile->FileExists(*AlternatePackageName))
		{
			return;
		}
	}

	// send the filename over (cast away const here because we know this << will not modify the string)
	FNetworkFileArchive Payload(NFS_Messages::SyncFile);
	Payload << (FString&)Filename;

	UE_LOGF(LogNetworkPlatformFile, Log, "Requesting file %ls", *Filename);

	StartTime = FPlatformTime::Seconds();

	// allocate array reader on the heap, because the SyncWriteFile function will delete it
	FArrayReader Response;
	if (!SendPayloadAndReceiveResponse(Payload, Response))
	{
		UE_LOGF(LogNetworkPlatformFile, Fatal, "Receive failure!");
		return;
	}
	ThisTime = 1000.0f * float(FPlatformTime::Seconds() - StartTime);
	TotalNetworkSyncTime += ThisTime;
	//UE_LOGF(LogNetworkPlatformFile, Display, "Send and receive %6.2fms", ThisTime);

	StartTime = FPlatformTime::Seconds();

	FString ReplyFile;
	Response << ReplyFile;
	ConvertServerFilenameToClientFilename(ReplyFile);
	check((ReplyFile == Filename) || (FPackageName::IsPackageFilename(Filename) && (FPathViews::GetBaseFilename(Filename) == FPathViews::GetBaseFilename(ReplyFile))) );

	// get the server file timestamp
	FDateTime ServerTimeStamp;
	Response << ServerTimeStamp;

	if (ServerTimeStamp != FDateTime::MinValue())  // if the file didn't actually exist on the server, don't create a zero byte file
	{
		UE_LOGF(LogNetworkPlatformFile, Verbose, "Succeeded in getting %ls from server", *Filename);
	}
	else
	{
		UE_LOGF(LogNetworkPlatformFile, Verbose, "File not found %ls from server", *Filename);
	}

	// write the file in chunks, synchronously
	SyncWriteFile(&Response, ReplyFile, ServerTimeStamp, *InnerPlatformFile);

	int32 NumUnsolictedFiles;
	Response << NumUnsolictedFiles;

	if (NumUnsolictedFiles)
	{
		check( FinishedAsyncNetworkReadUnsolicitedFiles == NULL );
		check( FinishedAsyncWriteUnsolicitedFiles == NULL );
		TotalUnsolicitedPackages += NumUnsolictedFiles;

		// There are problems with some platforms where writing files can be slow.  In this situation,
		// the async writing of files can become a bottleneck and 70+ async writes can be spawned as
		// runnables and at any given moment an indeterminate number of them could be trying to write
		// at the same time.  Because this is undesirable, the async writing of unsolicited files is disable
		// for the time being until it can be changed to be more efficient at writing and can be
		// throttled for simultaneous write attempts.
		constexpr bool bWriteUnsolicitedAsync = false;

		if (bWriteUnsolicitedAsync)
		{
			FinishedAsyncNetworkReadUnsolicitedFiles = new FScopedEvent;
			FinishedAsyncWriteUnsolicitedFiles = new FScopedEvent;
			AsyncReadUnsolicitedFiles(NumUnsolictedFiles, *this, *InnerPlatformFile, ServerEngineDir, ServerProjectDir, ServerEnginePlatformExtensionsDir, ServerProjectPlatformExtensionsDir, FinishedAsyncNetworkReadUnsolicitedFiles, FinishedAsyncWriteUnsolicitedFiles);
		}
		else
		{
			for (int32 Index = 0; Index < NumUnsolictedFiles; Index++)
			{
				FArrayReader UnsolictedResponse;
				if (!ReceiveResponse(UnsolictedResponse))
				{
					UE_LOGF(LogNetworkPlatformFile, Fatal, "Receive failure!");
					return;
				}
				FString UnsolictedReplyFile;
				UnsolictedResponse << UnsolictedReplyFile;

				if (!UnsolictedReplyFile.IsEmpty())
				{
					ConvertServerFilenameToClientFilename(UnsolictedReplyFile);
					// get the server file timestamp
					FDateTime UnsolictedServerTimeStamp;
					UnsolictedResponse << UnsolictedServerTimeStamp;

					// write the file by pulling out of the FArrayReader
					SyncWriteFile(&UnsolictedResponse, UnsolictedReplyFile, UnsolictedServerTimeStamp, *InnerPlatformFile);
				}
			}
		}
	}
	
	ThisTime = 1000.0f * float(FPlatformTime::Seconds() - StartTime);
	TotalWriteTime += ThisTime;
	//UE_LOGF(LogNetworkPlatformFile, Display, "Write file to local %6.2fms", ThisTime);
}

static FString NetworkPlatformFileEndChop("/");
void FNetworkPlatformFile::MakeStandardNetworkFilename(FString& Filename)
{
	FPaths::MakeStandardFilename(Filename);
	Filename.RemoveFromEnd(NetworkPlatformFileEndChop, ESearchCase::CaseSensitive);
}

bool FNetworkPlatformFile::IsInLocalDirectory(const FString& Filename)
{
	// cache the directory of the input file
	FString Directory = FPaths::GetPath(Filename);

	// look if the file is in a local directory
	for (int32 DirIndex = 0; DirIndex < LocalDirectories.Num(); DirIndex++)
	{
		if (Directory.StartsWith(LocalDirectories[DirIndex]))
		{
			return true;
		}
	}

	// if not local, talk to the server
	return false;
}

void FNetworkPlatformFile::PerformHeartbeat()
{
	// send the filename over (cast away const here because we know this << will not modify the string)
	FNetworkFileArchive Payload(NFS_Messages::Heartbeat);

	

	// send the filename over
	FArrayReader Response;
	{
		FScopeLock ScopeLock(&SynchronizationObject);
		if (!SendPayloadAndReceiveResponse(Payload, Response))
		{
			return;
		}
	}

	// get any files that have been modified on the server - 
	TArray<FString> UpdatedFiles;
	Response << UpdatedFiles;

	// delete any outdated files from the client
	// @todo: This may need a critical section around all calls to LowLevel in the other functions
	// because we don't want to delete files while other threads are using them!

	TArray<FString> PackageNames;
	for (int32 FileIndex = 0; FileIndex < UpdatedFiles.Num(); FileIndex++)
	{
		// clean up the linkers for this package
		FString LocalFileName = UpdatedFiles[FileIndex];
		ConvertServerFilenameToClientFilename( LocalFileName );

		UE_LOGF(LogNetworkPlatformFile, Log, "Server updated file '%ls', deleting local copy %ls", *UpdatedFiles[FileIndex], *LocalFileName);

		FString PackageName;
		if (FPackageName::TryConvertFilenameToLongPackageName(LocalFileName, PackageName))
		{
			PackageNames.Add(PackageName);
		}
		else
		{
			UE_LOGF(LogNetworkPlatformFile, Log, "Unable to convert filename to package name %ls", *LocalFileName);
		}

		OnFileUpdated(LocalFileName);
	}

	if ( PackageNames.Num() > 0 )
	{
		FCoreUObjectDelegates::NetworkFileRequestPackageReload.ExecuteIfBound(PackageNames);
	}
}

void FNetworkPlatformFile::OnFileUpdated(const FString& LocalFileName)
{
	if (InnerPlatformFile->FileExists(*LocalFileName) && InnerPlatformFile->DeleteFile(*LocalFileName) == false)
	{
		UE_LOGF(LogNetworkPlatformFile, Error, "Failed to delete %ls, someone is probably accessing without FNetworkPlatformFile, or we need better thread protection", *LocalFileName);
	}
	CachedLocalFiles.Remove(LocalFileName);
	ServerFiles.AddFileOrDirectory(LocalFileName, FDateTime::UtcNow());
}

void FNetworkPlatformFile::ConvertServerFilenameToClientFilename(FString& FilenameToConvert, const FString& InServerEngineDir, const FString& InServerProjectDir, const FString& InServerEnginePlatformExtensionsDir, const FString& InServerProjectPlatformExtensionsDir)
{
	if (FilenameToConvert.StartsWith(InServerEngineDir))
	{
		FilenameToConvert = FilenameToConvert.Replace(*InServerEngineDir, *(FPaths::EngineDir()));
	}
	else if (FilenameToConvert.StartsWith(InServerProjectDir))
	{
		FilenameToConvert = FilenameToConvert.Replace(*InServerProjectDir, *(FPaths::ProjectDir()));
	}
	else if (FilenameToConvert.StartsWith(InServerEnginePlatformExtensionsDir))
	{
		FilenameToConvert = FilenameToConvert.Replace(*InServerEnginePlatformExtensionsDir, *(UnsafeEnginePlatformExtensionDir()));
	}
	else if (FilenameToConvert.StartsWith(InServerProjectPlatformExtensionsDir))
	{
		FilenameToConvert = FilenameToConvert.Replace(*InServerProjectPlatformExtensionsDir, *(UnsafeProjectPlatformExtensionDir()));
	}
}

void FNetworkPlatformFile::Tick() 
{
	// try send a heart beat every 5 seconds as long as we are not async loading
	static double StartTime = FPlatformTime::Seconds();

	bool bShouldPerformHeartbeat = true;
	if ((FPlatformTime::Seconds() - StartTime) > HeartbeatFrequency && HeartbeatFrequency >= 0 )
	{


		if (IsAsyncLoading() && bShouldPerformHeartbeat)
		{
			bShouldPerformHeartbeat = false;
		}

		{
			FScopeLock S(&SynchronizationObject);
			if (FinishedAsyncNetworkReadUnsolicitedFiles && bShouldPerformHeartbeat)
			{
				if ( FinishedAsyncNetworkReadUnsolicitedFiles->IsReady() )
				{
					delete FinishedAsyncNetworkReadUnsolicitedFiles;
					FinishedAsyncNetworkReadUnsolicitedFiles = nullptr;
				}
				else
				{
					bShouldPerformHeartbeat = false;
				}
			}
		}

		if ( bShouldPerformHeartbeat )
		{
			StartTime = FPlatformTime::Seconds();

			//DeleteLoaders();
			PerformHeartbeat();
		}
	}
}

bool FNetworkPlatformFile::Exec_Runtime(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar )
{
	if (FParse::Command(&Cmd, TEXT("networkfile")))
	{
		if ( FParse::Command(&Cmd, TEXT("stats")))
		{

			Ar.Logf(TEXT("Network platform file %s stats\n"
				"TotalWriteTime \t%fms \n"
				"TotalNetworkSyncTime \t%fms \n"
				"TotalTimeSpentInUnsolicitedPackages \t%fms \n"
				"TotalWaitForAsyncUnsolicitedPackages \t%fms \n"
				"TotalFilesSynced \t%d \n"
				"TotalFilesFoundLocally \t%d\n"
				"TotalUnsolicitedPackages \t%d \n"
				"UnsolicitedPackagesHits \t%d \n"
				"UnsolicitedPackageWaits \t%d \n"),
				GetTypeName(),
				TotalWriteTime,
				TotalNetworkSyncTime,
				TotalTimeSpentInUnsolicitedPackages,
				TotalWaitForAsyncUnsolicitedPackages,
				TotalFilesSynced,
				TotalFilesFoundLocally,
				TotalUnsolicitedPackages, 
				UnsolicitedPackagesHits,
				UnsolicitedPackageWaits);

			// there could be multiple network platform files so let them all report their stats
			return false;
		}
	}
	return false;
}

/**
 * Module for the network file
 */
class FNetworkFileModule : public IPlatformFileModule
{
public:
	virtual IPlatformFile* GetPlatformFile() override
	{
		static TUniquePtr<IPlatformFile> AutoDestroySingleton = MakeUnique<FNetworkPlatformFile>();
		return AutoDestroySingleton.Get();
	}
};
IMPLEMENT_MODULE(FNetworkFileModule, NetworkFile);
