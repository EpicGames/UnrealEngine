// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/SpscQueue.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/CriticalSection.h"
#include "HAL/Event.h"
#include "Logging/LogMacros.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/CoreMisc.h" // included for FSelfRegisteringExec
#include "Misc/DateTime.h"
#include "NetworkMessage.h"
#include "ServerTOC.h"
#include "Templates/SharedPointer.h"

class FArrayReader;
class FOutputDevice;
class FScopedEvent;
struct FPackageFileVersion;

namespace UE { namespace Cook
{
	class FCookOnTheFlyMessage;
	class ICookOnTheFlyServerConnection;
}}

DECLARE_LOG_CATEGORY_EXTERN(LogNetworkPlatformFile, Log, All);


/**
 * Wrapper to redirect the low level file system to a server
 */
class FNetworkPlatformFile : public IPlatformFile, public FSelfRegisteringExec
{
	friend class FAsyncFileSync;
	friend void ReadUnsolicitedFile(int32 InNumUnsolictedFiles, FNetworkPlatformFile& InNetworkFile, IPlatformFile& InInnerPlatformFile,  FString& InServerEngineDir, FString& InServerProjectDir);

protected:
	/**
	 * Initialize network platform file give the specified host IP
	 *
	 * @param Inner Inner platform file
	 * @param HostIP host IP address
	 * @return true if the initialization succeeded, false otherwise
	 */
	NETWORKFILE_API virtual bool InitializeInternal(IPlatformFile* Inner, const TCHAR* HostIP);

	NETWORKFILE_API virtual void OnFileUpdated(const FString& LocalFilename);

public:
	

	static const TCHAR* GetTypeName()
	{
		return TEXT("NetworkFile");
	}

	/** Constructor */
	NETWORKFILE_API FNetworkPlatformFile();

	/** Destructor */
	NETWORKFILE_API virtual ~FNetworkPlatformFile();

	NETWORKFILE_API virtual bool ShouldBeUsed(IPlatformFile* Inner, const TCHAR* CmdLine) const override;
	NETWORKFILE_API virtual bool Initialize(IPlatformFile* Inner, const TCHAR* CmdLine) override;
	NETWORKFILE_API virtual void InitializeAfterSetActive() override;

	virtual IPlatformFile* GetLowerLevel() override
	{
		return InnerPlatformFile;
	}
	virtual void SetLowerLevel(IPlatformFile* NewLowerLevel) override
	{
		InnerPlatformFile = NewLowerLevel;
	}
	virtual void GetTimeStampPair(const TCHAR* PathA, const TCHAR* PathB, FDateTime& OutTimeStampA, FDateTime& OutTimeStampB)
	{
		OutTimeStampA = GetTimeStamp(PathA);
		OutTimeStampB = GetTimeStamp(PathB);

		if (GetLowerLevel() && OutTimeStampA == FDateTime::MinValue() && OutTimeStampB == FDateTime::MinValue())
		{
			GetLowerLevel()->GetTimeStampPair(PathA, PathB, OutTimeStampA, OutTimeStampB);
		}
	}

	virtual const TCHAR* GetName() const override
	{
		return FNetworkPlatformFile::GetTypeName();
	}

	virtual bool IsUsable()
	{
		return bIsUsable;
	}

	virtual bool		FileExists(const TCHAR* Filename) override
	{
		FFileInfo Info;
		GetFileInfo(Filename, Info);
		return Info.FileExists;
	}
	virtual int64		FileSize(const TCHAR* Filename) override
	{
		FFileInfo Info;
		GetFileInfo(Filename, Info);
		return Info.Size;
	}
	NETWORKFILE_API virtual bool		DeleteFile(const TCHAR* Filename) override;
	virtual bool		IsReadOnly(const TCHAR* Filename) override
	{
		FFileInfo Info;
		GetFileInfo(Filename, Info);
		return Info.ReadOnly;
	}
	NETWORKFILE_API virtual bool		MoveFile(const TCHAR* To, const TCHAR* From) override;
	NETWORKFILE_API virtual bool		SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue) override;
	virtual FDateTime	GetTimeStamp(const TCHAR* Filename) override
	{
		FFileInfo Info;
		GetFileInfo(Filename, Info);
		return Info.TimeStamp;
	}
	NETWORKFILE_API virtual void		SetTimeStamp(const TCHAR* Filename, FDateTime DateTime) override;
	virtual FDateTime	GetAccessTimeStamp(const TCHAR* Filename) override
	{
		FFileInfo Info;
		GetFileInfo(Filename, Info);
		return Info.AccessTimeStamp;
	}
	virtual FString	GetFilenameOnDisk(const TCHAR* Filename) override
	{
		return Filename;
	}
	NETWORKFILE_API virtual IFileHandle*	OpenRead(const TCHAR* Filename, bool bAllowWrite = false) override;
	NETWORKFILE_API virtual IFileHandle*	OpenWrite(const TCHAR* Filename, bool bAppend = false, bool bAllowRead = false) override;
	NETWORKFILE_API virtual bool		DirectoryExists(const TCHAR* Directory) override;
	NETWORKFILE_API virtual bool		CreateDirectoryTree(const TCHAR* Directory) override;
	NETWORKFILE_API virtual bool		CreateDirectory(const TCHAR* Directory) override;
	NETWORKFILE_API virtual bool		DeleteDirectory(const TCHAR* Directory) override;

	NETWORKFILE_API virtual FFileStatData GetStatData(const TCHAR* FilenameOrDirectory) override;

	NETWORKFILE_API virtual bool		IterateDirectory(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor) override;
	NETWORKFILE_API virtual bool		IterateDirectoryRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor) override;

	NETWORKFILE_API virtual bool		IterateDirectoryStat(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitor& Visitor) override;
	NETWORKFILE_API virtual bool		IterateDirectoryStatRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitor& Visitor) override;

	NETWORKFILE_API virtual bool		DeleteDirectoryRecursively(const TCHAR* Directory) override;
	NETWORKFILE_API virtual bool		CopyFile(const TCHAR* To, const TCHAR* From, EPlatformFileRead ReadFlags = EPlatformFileRead::None, EPlatformFileWrite WriteFlags = EPlatformFileWrite::None) override;

	NETWORKFILE_API virtual FString ConvertToAbsolutePathForExternalAppForRead( const TCHAR* Filename ) override;
	NETWORKFILE_API virtual FString ConvertToAbsolutePathForExternalAppForWrite( const TCHAR* Filename ) override;

	NETWORKFILE_API virtual bool SendMessageToServer(const TCHAR* Message, IPlatformFile::IFileServerMessageHandler* Handler) override;


	NETWORKFILE_API virtual void Tick() override;


	
	NETWORKFILE_API virtual bool SendPayloadAndReceiveResponse(TArray<uint8>& In, TArray<uint8>& Out);
	NETWORKFILE_API virtual bool ReceiveResponse(TArray<uint8>& Out);

	NETWORKFILE_API bool SendReadMessage(uint8* Destination, int64 BytesToRead);
	NETWORKFILE_API bool SendWriteMessage(const uint8* Source, int64 BytesToWrite);

	static NETWORKFILE_API void ConvertServerFilenameToClientFilename(FString& FilenameToConvert, const FString& InServerEngineDir, const FString& InServerProjectDir, const FString& InServerEnginePlatformExtensionsDir, const FString& InServerProjectPlatformExtensionsDir);


	NETWORKFILE_API virtual FString GetVersionInfo() const;

protected:

	//////////////////////////////////////////////////////////////////////////
	// FSelfRegisteringExec interface
	NETWORKFILE_API virtual bool Exec_Runtime(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar ) override;


	/**
	 * Send a heartbeat message to the file server. This will tell it we are alive, as well as
	 * get back a list of files that have been updated on the server (so we can toss our copy)
	 */
	NETWORKFILE_API virtual void PerformHeartbeat();

	NETWORKFILE_API virtual void GetFileInfo(const TCHAR* Filename, FFileInfo& Info);

	/**
	 *	Convert the given filename from the server to the client version of it
	 *	NOTE: Potentially modifies the input FString!!!!
	 *
	 *	@param	FilenameToConvert		Upon input, the server version of the filename. After the call, the client version
	 */
	NETWORKFILE_API virtual void ConvertServerFilenameToClientFilename(FString& FilenameToConvert);

	NETWORKFILE_API virtual void FillGetFileList(FNetworkFileArchive& Payload);

	NETWORKFILE_API virtual void ProcessServerInitialResponse(FArrayReader& InResponse, FPackageFileVersion& OutServerPackageVersion, int32& OutServerPackageLicenseeVersion);
	NETWORKFILE_API virtual void ProcessServerCachedFilesResponse(FArrayReader& InReponse, const FPackageFileVersion& ServerPackageVersion, const int32 ServerPackageLicenseeVersion );

private:

	/**
	 * Returns whether the passed in extension is a video
	 * extension. Extensions with and without trailing dots are supported.
	 *
	 * @param	Extension to test.
	 * @return	True if Ext is a video extension.  e.g. .mp4
	 */
	static NETWORKFILE_API bool IsMediaExtension(const TCHAR* Ext);

	/**
	 * Returns whether the passed in extension is a an additional (but non asset) cooked file.
	 *
	 * @param	Extension to test.
	 * @return	True if Ext is a additional cooked file.  e.g. .ubulk, .ufont
	 */
	static NETWORKFILE_API bool IsAdditionalCookedFileExtension(const TCHAR* Ext);

	/**
	 * @return true if the path exists in a directory that should always use the local filesystem
	 */
	NETWORKFILE_API bool IsInLocalDirectory(const FString& Filename);

	/**
	 * Given a filename, make sure the file exists on the local filesystem
	 */
	NETWORKFILE_API void EnsureFileIsLocal(const FString& Filename);

	NETWORKFILE_API void OnCookOnTheFlyMessage(const UE::Cook::FCookOnTheFlyMessage& Message);

protected:
	/**
	* Does normal path standardization, and also any extra modifications to make string comparisons against
	* the internal directory list work properly.
	*/
	NETWORKFILE_API void MakeStandardNetworkFilename(FString& Filename);

protected:

	/** The file interface to read/write local files with */
	IPlatformFile*		InnerPlatformFile;

	/** This keeps track of what files have been "EnsureFileIsLocal'd" */
	TSet<FString>		CachedLocalFiles;

	/** The server engine dir */
	FString ServerEngineDir;

	/** The server game dir */
	FString ServerProjectDir;

	/** The server engine platform extensions dir */
	FString ServerEnginePlatformExtensionsDir;

	/** The server project platform extensions dir */
	FString ServerProjectPlatformExtensionsDir;

	/** This is the "TOC" of the server */
	FServerTOC ServerFiles;

	/** Set of directories that should use the local filesystem */
	TArray<FString> LocalDirectories;

	FCriticalSection	SynchronizationObject;
	bool				bIsUsable;
	int32				FileServerPort;

	// the connection flags are passed to the server during GetFileList
	// the server may cache them
	EConnectionFlags	ConnectionFlags;
	// Frequency to send heartbeats to server in seconds set to negative number to disable
	float HeartbeatFrequency;


	// some stats for messuring network platform file performance
	double TotalWriteTime; // total non async time spent writing to disk
	double TotalNetworkSyncTime; // total non async time spent syncing to network
	int32 TotalFilesSynced; // total number files synced from network
	int32 TotalUnsolicitedPackages; // total number unsolicited files synced  
	int32 TotalFilesFoundLocally;
	int32 UnsolicitedPackagesHits; // total number of hits from waiting on unsolicited packages
	int32 UnsolicitedPackageWaits; // total number of waits on unsolicited packages
	double TotalTimeSpentInUnsolicitedPackages; // total time async processing unsolicited packages
	double TotalWaitForAsyncUnsolicitedPackages; // total time spent waiting for unsolicited packages



private:

	/* Unsolicitied files events */
	FScopedEvent *FinishedAsyncNetworkReadUnsolicitedFiles;
	FScopedEvent *FinishedAsyncWriteUnsolicitedFiles;

	TSharedPtr<UE::Cook::ICookOnTheFlyServerConnection> Connection;
	TSpscQueue<TArray<uint8>> PendingPayloads;
	FEventRef NewPayloadEvent;

	static NETWORKFILE_API FString MP4Extension;
	static NETWORKFILE_API FString BulkFileExtension;
	static NETWORKFILE_API FString ExpFileExtension;
	static NETWORKFILE_API FString FontFileExtension;
};

class SOCKETS_API FNetworkFileHandle : public IFileHandle
{
	FNetworkPlatformFile&		Network;
	FString					Filename;
	int64						FilePos;
	int64						Size;
	bool						bWritable;
	bool						bReadable;
public:

	FNetworkFileHandle(FNetworkPlatformFile& InNetwork, const TCHAR* InFilename, int64 InFilePos, int64 InFileSize, bool bWriting)
		: Network(InNetwork)
		, Filename(InFilename)
		, FilePos(InFilePos)
		, Size(InFileSize)
		, bWritable(bWriting)
		, bReadable(!bWriting)
	{
	}

	virtual int64		Tell() override
	{
		return FilePos;
	}
	virtual bool		Seek(int64 NewPosition) override
	{
		if (NewPosition >= 0 && NewPosition <= Size)
		{
			FilePos = NewPosition;
			return true;
		}
		return false;
	}
	virtual bool		SeekFromEnd(int64 NewPositionRelativeToEnd = 0) override
	{
		return Seek(Size + NewPositionRelativeToEnd);
	}
	virtual bool		Read(uint8* Destination, int64 BytesToRead) override
	{
		bool Result = false;
		if (bReadable && BytesToRead >= 0 && BytesToRead + FilePos <= Size)
		{
			if (BytesToRead == 0)
			{
				Result = true;
			}
			else
			{
				Result = Network.SendReadMessage(Destination, BytesToRead);
				if (Result)
				{
					FilePos += BytesToRead;
				}
			}
		}
		return Result;
	}
	virtual bool		Write(const uint8* Source, int64 BytesToWrite) override
	{
		bool Result = false;
		if (bWritable && BytesToWrite >= 0)
		{
			if (BytesToWrite == 0)
			{
				Result = true;
			}
			else
			{
				Result = Network.SendWriteMessage(Source, BytesToWrite);
				if (Result)
				{
					FilePos += BytesToWrite;
					Size = FMath::Max<int64>(FilePos, Size);
				}
			}
		}
		return Result;
	}
	virtual bool		Flush(const bool bFullFlush = false) override
	{
		return false;
	}
	virtual bool		Truncate(int64 NewSize) override
	{
		return false;
	}
};



